// Copyright Epic Games, Inc. All Rights Reserved.

namespace UnrealBuildTool.Rules
{
	public class MovieRenderPipelineTmvEncoder : ModuleRules
	{
		public MovieRenderPipelineTmvEncoder(ReadOnlyTargetRules Target) : base(Target)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"Core",
					"CoreUObject",
					"Engine",
					"ImageCore",
					"MovieRenderPipelineCore",
					"OpenColorIO",
					"RHI",
					"SlateCore",
					"TmvMedia",
				}
			);

			// Todo: other platforms
			if (Target.Platform == UnrealTargetPlatform.Win64)
			{
				PrivateDependencyModuleNames.Add("ApvMedia");
			}
		}
	}
}
