// Copyright Epic Games, Inc. All Rights Reserved.

#include "Graph/MovieGraphTmvEncoderNode.h"

#include "ApvMediaTmvEncoderOptions.h"
#include "Graph/MovieGraphBlueprintLibrary.h"
#include "Graph/MovieGraphConfig.h"
#include "Graph/MovieGraphFilenameResolveParams.h"
#include "Graph/MovieGraphOCIOHelper.h"
#include "Graph/MovieGraphPipeline.h"
#include "Graph/Nodes/MovieGraphBurnInNode.h"
#include "Graph/Nodes/MovieGraphGlobalOutputSettingNode.h"
#include "ImageWriteTask.h"
#include "MoviePipelineImageQuantization.h"
#include "MoviePipelineTelemetry.h"
#include "SampleBuffer.h"
#include "SampleConverter/TmvMediaFrameMipImageBuffer.h"
#include "SampleConverter/TmvMediaFrameMipPixelDataBuffer.h"
#include "Styling/AppStyle.h"
#include "Transcoder/TmvMediaFrameConverter.h"
#include "Transcoder/TmvMediaFrameMips.h"
#include "Transcoder/TmvMediaFrameProducer.h"
#include "Transcoder/TmvMediaTranscodeJob.h"
#include "Transcoder/TmvMediaTranscodeJobBuilder.h"
#include "Transcoder/TmvMediaTranscodeList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieGraphTmvEncoderNode)

namespace UE::TmvMediaUtils::Encoder
{
	ETmvMediaTranscodeOutputFormat GetTranscodeOutputFormat(ETmvEncoderNodeOutputType InNodeOutputFormat)
	{
		switch (InNodeOutputFormat)
		{
		case ETmvEncoderNodeOutputType::Apv1ImageSequence:
			return ETmvMediaTranscodeOutputFormat::FileSequence;
		case ETmvEncoderNodeOutputType::TmvVideoFile:
		default:
			return ETmvMediaTranscodeOutputFormat::Container;
		}
	}

	bool InitFrameMips(FTmvMediaFrameMips& InFrameMips, TUniquePtr<FImagePixelData>&& InMip0, bool bInGenerateMips)
	{
		if (!InMip0.IsValid())
		{
			return false;
		}

		FImageView Mip0View = InMip0->GetImageView();	// make view before moving the pointer.

		InFrameMips.MipBuffers.Reset();
		InFrameMips.MipBuffers.Add(MakeShared<FTmvMediaFrameMipPixelDataBuffer>(MoveTemp(InMip0), 0));

		// Directly use ImageCore to generate the mips on the cpu. (It's reasonably fast.)
		if (bInGenerateMips && (Mip0View.SizeX > 1 || Mip0View.SizeY > 1))
		{
			FImageView PreviousMipView = Mip0View;

			// Protecting against infinite loop.
			constexpr int32 MaxMips = 32;

			while (InFrameMips.MipBuffers.Num() < MaxMips)
			{
				int32 MipSizeX = FMath::Max(PreviousMipView.SizeX/2, 1);
				int32 MipSizeY = FMath::Max(PreviousMipView.SizeY/2, 1);

				TSharedPtr<FImage> NewMip = MakeShared<FImage>(MipSizeX, MipSizeY, 1, Mip0View.Format, Mip0View.GammaSpace);
				{
					TRACE_CPUPROFILER_EVENT_SCOPE(TmvMediaTranscode::GenerateMip);
					FImageCore::ResizeImage(PreviousMipView, *NewMip);
				}
				int32 MipLevel = InFrameMips.MipBuffers.Num();
				InFrameMips.MipBuffers.Add(MakeShared<FTmvMediaFrameMipImageBuffer>(NewMip, MipLevel));

				PreviousMipView = *NewMip;

				if (MipSizeX == 1 && MipSizeY == 1)
				{
					break;
				}
			}
		}

		return true;
	}
	
	/** Determines the time code that the movie should be started at */
	FTimecode GetStartTimeCode(
		const FMovieGraphTraversalContext* InTraversalContext,
		const UMovieGraphGlobalOutputSettingNode* InOutputSetting)
	{
		if (InOutputSetting->bOverride_CustomTimecodeStart)
		{
			return InOutputSetting->CustomTimecodeStart;
		}
		
		// This is the frame number on the global time, can have overlaps (between encoders) or repeats when using handle frames/slowmo.
		return InTraversalContext->Time.RootTimeCode;
	}
}

UMovieGraphTmvEncoderNode::UMovieGraphTmvEncoderNode()
{
	FileNameFormat = TEXT("{sequence_name}.{layer_name}");
}

#if WITH_EDITOR
FText UMovieGraphTmvEncoderNode::GetNodeTitle(const bool bGetDescriptive) const
{
	static const FText NodeName = NSLOCTEXT("MovieGraphNodes", "NodeName_ApvTmv", "APV TMV");
	return NodeName;
}

FText UMovieGraphTmvEncoderNode::GetMenuCategory() const
{
	return NSLOCTEXT("MovieGraphNodes", "TmvEncoderNode_Category", "Output Type");
}

FText UMovieGraphTmvEncoderNode::GetKeywords() const
{
	static const FText Keywords = NSLOCTEXT("MovieGraphNodes", "TmvEncoderNode_Keywords", "tmv encoder apv exr movie video");
	return Keywords;
}

FLinearColor UMovieGraphTmvEncoderNode::GetNodeTitleColor() const
{
	static const FLinearColor NodeColor = FLinearColor(0.047f, 0.654f, 0.537f);
	return NodeColor;
}

FSlateIcon UMovieGraphTmvEncoderNode::GetIconAndTint(FLinearColor& OutColor) const
{
	static const FSlateIcon Icon = FSlateIcon(FAppStyle::GetAppStyleSetName(), "LevelEditor.Tabs.Cinematics");

	OutColor = FLinearColor::White;
	return Icon;
}

bool UMovieGraphTmvEncoderNode::CanEditChange( const FProperty* InProperty ) const
{
	if (!Super::CanEditChange(InProperty))
	{
		return false;
	}

	// Special logic to prevent editing color transform properties if ocio config is enabled.
	const FName PropertyName = InProperty ? InProperty->GetFName() : NAME_None;
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMovieGraphTmvEncoderNode, DestinationColorSpace)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UMovieGraphTmvEncoderNode, DestinationEncoding)
		|| PropertyName == GET_MEMBER_NAME_CHECKED(UMovieGraphTmvEncoderNode, ReferenceWhite))
	{
		return !IsOcioEnabledEditor();
	}

	return true;
}

void UMovieGraphTmvEncoderNode::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UMovieGraphTmvEncoderNode, OutputType))
	{
		static const FString FrameNumberToken = TEXT("{frame_number}");

		if (OutputType == ETmvEncoderNodeOutputType::TmvVideoFile)
		{
			// Remove {frame_number} and its preceding separator (., -, _) if present anywhere in the format.
			const int32 TokenIndex = FileNameFormat.Find(FrameNumberToken);
			if (TokenIndex != INDEX_NONE)
			{
				int32 RemoveStart = TokenIndex;
				// Also remove a preceding separator character if present.
				if (RemoveStart > 0)
				{
					const TCHAR PrecedingChar = FileNameFormat[RemoveStart - 1];
					if (PrecedingChar == TEXT('.') || PrecedingChar == TEXT('-') || PrecedingChar == TEXT('_'))
					{
						RemoveStart--;
					}
				}
				FileNameFormat.RemoveAt(RemoveStart, TokenIndex - RemoveStart + FrameNumberToken.Len());
			}
		}
		else
		{
			// Append .{frame_number} if the token is not already present.
			if (!FileNameFormat.Contains(FrameNumberToken))
			{
				FileNameFormat += TEXT(".{frame_number}");
			}
		}

		OnNodeChangedDelegate.Broadcast(this);
	}
}
#endif // WITH_EDITOR

TUniquePtr<MovieRenderGraph::IVideoCodecWriter> UMovieGraphTmvEncoderNode::Initialize_GameThread(const FMovieGraphVideoNodeInitializationContext& InInitializationContext)
{
	bool bIncludeCDOs = true;
	constexpr bool bExactMatch = true;
	UMovieGraphGlobalOutputSettingNode* OutputSetting =
		InInitializationContext.EvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

	bIncludeCDOs = false;
	const UMovieGraphTmvEncoderNode* EvaluatedNode = Cast<UMovieGraphTmvEncoderNode>(
		InInitializationContext.EvaluatedConfig->GetSettingForBranch(GetClass(), FName(InInitializationContext.PassData->Key.RootBranchName), bIncludeCDOs, bExactMatch));
	checkf(EvaluatedNode, TEXT("TMV Encoder node could not be found in the graph in branch [%s]."), *InInitializationContext.PassData->Key.RootBranchName.ToString());

	const FFrameRate SourceFrameRate = InInitializationContext.Pipeline->GetDataSourceInstance()->GetDisplayRate();
	const FFrameRate EffectiveFrameRate = UMovieGraphBlueprintLibrary::GetEffectiveFrameRate(OutputSetting, SourceFrameRate);

	TUniquePtr<FTmvCodecWriter> NewWriter = MakeUnique<FTmvCodecWriter>();

	FTmvMediaTranscodeJobSettings JobSettings;

	JobSettings.OutputPath.Path = FPaths::GetPath(InInitializationContext.FileName);

	if (EvaluatedNode->OutputType == ETmvEncoderNodeOutputType::Apv1ImageSequence)
	{
		// For file sequence, resolve the filename format ourselves, preserving {frame_number} as a literal token
		// so the transcode pipeline's MakeFrameFilename can resolve it with the correct separator and placement.
		// The MRG base class strips {frame_number} before resolution, but we need it for per-frame file naming.
		const TMap<FString, FString> FormatOverrides = {
			{TEXT("camera_name"), InInitializationContext.PassData->Key.CameraName},
			{TEXT("render_pass"), InInitializationContext.PassData->Key.RendererName},
			{TEXT("frame_number"), TEXT("{frame_number}")},
			{TEXT("frame_number_rel"), TEXT("{frame_number}")},
			{TEXT("frame_number_shot"), TEXT("{frame_number}")},
			{TEXT("frame_number_shot_rel"), TEXT("{frame_number}")},
		};

		FMovieGraphFilenameResolveParams ResolveParams = FMovieGraphFilenameResolveParams::MakeResolveParams(
			InInitializationContext.PassData->Key, InInitializationContext.Pipeline,
			InInitializationContext.EvaluatedConfig, *InInitializationContext.TraversalContext, FormatOverrides);
		ResolveParams.FileNameOverride = EvaluatedNode->FileNameFormat;

		FMovieGraphResolveArgs ResolveArgs;
		JobSettings.OutputBaseName = UMovieGraphBlueprintLibrary::ResolveFormatArguments(EvaluatedNode->FileNameFormat, ResolveParams, ResolveArgs);
	}
	else
	{
		JobSettings.OutputBaseName = FPaths::GetBaseFilename(InInitializationContext.FileName);
	}

	JobSettings.ZeroPadFrameNumbers = OutputSetting->ZeroPadFrameNumbers;
	JobSettings.FrameRate = EffectiveFrameRate;
	JobSettings.bEnableMipMapping = EvaluatedNode->bEnableMipMapping;
	JobSettings.OutputFormat = UE::TmvMediaUtils::Encoder::GetTranscodeOutputFormat(EvaluatedNode->OutputType);

	if (EvaluatedNode->OutputType == ETmvEncoderNodeOutputType::TmvVideoFile)
	{
		JobSettings.Muxer.Name = TEXT("Tmv");
		static const FTimecode TimeCodeZero = FTimecode();
		const FTimecode StartTimeCode = UE::TmvMediaUtils::Encoder::GetStartTimeCode(InInitializationContext.TraversalContext, OutputSetting);
		if (StartTimeCode != TimeCodeZero)
		{
			JobSettings.bEnableStartTimecodeOverride = true;
			JobSettings.StartTimecodeOverride = StartTimeCode;
		}
	}

	// Construct APV encoder options from the flat node properties.
	TInstancedStruct<FTmvMediaEncoderOptions> ApvOptions;
	ApvOptions.InitializeAs<FApvMediaTmvEncoderOptions>();
	FApvMediaTmvEncoderOptions& Options = ApvOptions.GetMutable<FApvMediaTmvEncoderOptions>();
	Options.TileSize = EvaluatedNode->TileSize;
	Options.Profile = EvaluatedNode->Profile;
	Options.Band = EvaluatedNode->Band;
	Options.Preset = EvaluatedNode->Preset;
	Options.NumThreads = EvaluatedNode->NumThreads;
	// With Apv encoder, "color management" is always enabled because we need YuvMatrix and YuvMatrixRange.
	Options.bEnableColorManagement = true;
	// If ocio is enabled, we disable the builtin color transform
	Options.DestinationColorSpace = !EvaluatedNode->IsOcioEnabledRuntime() ? EvaluatedNode->DestinationColorSpace : ETextureColorSpace::TCS_None;
	Options.DestinationEncoding = !EvaluatedNode->IsOcioEnabledRuntime() ? EvaluatedNode->DestinationEncoding :  ETmvMediaEncoderEncoding::None;
	Options.ReferenceWhite = !EvaluatedNode->IsOcioEnabledRuntime() ? EvaluatedNode->ReferenceWhite : ETmvMediaEncoderReferenceWhite::None;
	Options.YuvMatrix = EvaluatedNode->YuvMatrix;
	Options.YuvMatrixRange = EvaluatedNode->YuvMatrixRange;

	// Assemble the pipeline through the job builder. Frames are pushed externally to the base
	// UTmvMediaFrameProducer, so we override the producer class and set the track info post-build.
	FTmvMediaTranscodeListItem JobItem;
	JobItem.Name = TEXT("MovieGraphTmvEncoder");
	JobItem.Settings = MoveTemp(JobSettings);
	JobItem.EncoderOptions = MoveTemp(ApvOptions);

	NewWriter->TranscodeJob.Reset(
		FTmvMediaTranscodeJobBuilder(JobItem)
			.WithFrameProducerClass(UTmvMediaFrameProducer::StaticClass())
			.OnPostBuild(FOnTranscodeJobBuilt::CreateLambda([EffectiveFrameRate](UTmvMediaTranscodeJob* InJob)
			{
				if (UTmvMediaFrameProducer* FrameProducer = InJob->GetStage<UTmvMediaFrameProducer>())
				{
					FTmvMediaFrameProducerTrackInfo TrackInfo;
					TrackInfo.FrameRate = EffectiveFrameRate;
					FrameProducer->SetVideoTrackInfo(TrackInfo);
				}
			}))
			.Build());

	NewWriter->TranscodeJob->Start(FApp::GetCurrentTime());

	CachedPipeline = InInitializationContext.Pipeline;

	return NewWriter;
}

bool UMovieGraphTmvEncoderNode::Initialize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	return true;
}

void UMovieGraphTmvEncoderNode::WriteFrame_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<FMovieGraphPassData>&& InCompositePasses, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FString& InBranchName)
{
	FTmvCodecWriter* CodecWriter = static_cast<FTmvCodecWriter*>(InWriter);
	if (!CodecWriter->TranscodeJob)
	{
		return;
	}

	bool bIncludeCDOs = false;
	constexpr bool bExactMatch = true;
	const UMovieGraphTmvEncoderNode* EvaluatedNode = Cast<UMovieGraphTmvEncoderNode>(
		InEvaluatedConfig->GetSettingForBranch(GetClass(), FName(InBranchName), bIncludeCDOs, bExactMatch));
	checkf(EvaluatedNode, TEXT("TMV Encoder node could not be found in the graph in branch [%s]."), *InBranchName);

	bIncludeCDOs = true;
	const UMovieGraphGlobalOutputSettingNode* OutputSettingNode =
		InEvaluatedConfig->GetSettingForBranch<UMovieGraphGlobalOutputSettingNode>(GlobalsPinName, bIncludeCDOs, bExactMatch);

	const UE::MovieGraph::FMovieGraphSampleState* GraphPayload = InPixelData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>();

	TArray<FPixelPreProcessor> PixelPreProcessors;

	// Run OCIO before quantizing. For highest accuracy, OCIO should run on the non-quantized pixel data.
#if WITH_OCIO
	if (FMovieGraphOCIOHelper::GenerateOcioPixelPreProcessor(GraphPayload, CachedPipeline.Get(), InEvaluatedConfig, EvaluatedNode->OCIOConfiguration, EvaluatedNode->OCIOContext, PixelPreProcessors))
	{
		PixelPreProcessors.Pop()(InPixelData);
	}
#endif

	// Intermediate processing will be float16 (linear) for TMV color conversion.
	constexpr int32 TargetBitDepth = 16;
	constexpr bool bConvertToSrgb = false;
	TUniquePtr<FImagePixelData> QuantizedPixelData = UE::MoviePipeline::QuantizeImagePixelDataToBitDepth(InPixelData, TargetBitDepth, nullptr, bConvertToSrgb);

	// Do a quick composite of renders/burn-ins.
	for (const FMovieGraphPassData& CompositePass : InCompositePasses)
	{
		// We don't need to copy the data here (even though it's being passed to a async system) because we already made a unique copy of the
		// burn in/widget data when we decided to composite it.
		switch (QuantizedPixelData->GetType())
		{
		case EImagePixelType::Color:
			PixelPreProcessors.Add(TAsyncCompositeImage<FColor>(CompositePass.Value->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float16:
			PixelPreProcessors.Add(TAsyncCompositeImage<FFloat16Color>(CompositePass.Value->MoveImageDataToNew()));
			break;
		case EImagePixelType::Float32:
			PixelPreProcessors.Add(TAsyncCompositeImage<FLinearColor>(CompositePass.Value->MoveImageDataToNew()));
			break;
		}
	}

	// This is done on the current thread for simplicity but the composite itself is parallelized.
	FImagePixelData* PixelData = QuantizedPixelData.Get();
	for (const FPixelPreProcessor& PreProcessor : PixelPreProcessors)
	{
		// PreProcessors are assumed to be valid.
		PreProcessor(PixelData);
	}

	if (UTmvMediaFrameConverter* FrameConverter = CodecWriter->TranscodeJob->GetStage<UTmvMediaFrameConverter>())
	{
		TUniquePtr<FTmvMediaFrameMips> FrameMips = MakeUnique<FTmvMediaFrameMips>();

		FrameMips->TimeInfo.FrameIndex = 0;
		FrameMips->TimeInfo.FrameIndexNoOffset = 0;
		
		if (const UE::MovieGraph::FMovieGraphSampleState* Payload = InPixelData->GetPayload<UE::MovieGraph::FMovieGraphSampleState>())
		{
			FrameMips->TimeInfo.FrameIndex = Payload->TraversalContext.Time.RootFrameNumber.Value;
			FrameMips->TimeInfo.FrameIndexNoOffset = Payload->TraversalContext.Time.OutputFrameNumber;	// Always relative to zero.
		}

		FrameMips->TimeInfo.FrameDuration = FTimespan::FromSeconds(CodecWriter->TranscodeJob->Settings.FrameRate.AsInterval());
		FrameMips->TimeInfo.FrameTime = FrameMips->TimeInfo.FrameDuration * FrameMips->TimeInfo.FrameIndex;
		
		UE::TmvMediaUtils::Encoder::InitFrameMips(*FrameMips, MoveTemp(QuantizedPixelData), CodecWriter->TranscodeJob->Settings.bEnableMipMapping);
		FrameConverter->ReceiveMips(CodecWriter->TranscodeJob.Get(), MoveTemp(FrameMips));
	}
}

void UMovieGraphTmvEncoderNode::BeginFinalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	const FTmvCodecWriter* CodecWriter = static_cast<FTmvCodecWriter*>(InWriter);
	if(CodecWriter && CodecWriter->TranscodeJob)
	{
		CodecWriter->TranscodeJob->RequestStop(FApp::GetCurrentTime(), ETmvMediaTranscodeJobStopReason::Completed);
	}
}

void UMovieGraphTmvEncoderNode::Finalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter)
{
	FTmvCodecWriter* CodecWriter = static_cast<FTmvCodecWriter*>(InWriter);
	if(CodecWriter && CodecWriter->TranscodeJob)
	{
		CodecWriter->TranscodeJob->Discard(FApp::GetCurrentTime());
		CodecWriter->TranscodeJob.Reset();
	}
}

const TCHAR* UMovieGraphTmvEncoderNode::GetFilenameExtension() const
{
	if (OutputType == ETmvEncoderNodeOutputType::TmvVideoFile)
	{
		return TEXT("tmv");
	}
	return TEXT("apv1");
}

bool UMovieGraphTmvEncoderNode::IsAudioSupported() const
{
	// todo: revisit when audio is implemented in Tmv container.
	return false;
}

void UMovieGraphTmvEncoderNode::UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const
{
	const ETmvMediaTranscodeOutputFormat TranscodeOutputFormat = UE::TmvMediaUtils::Encoder::GetTranscodeOutputFormat(OutputType);
	InTelemetry->bUsesTmvMovie |= (TranscodeOutputFormat == ETmvMediaTranscodeOutputFormat::Container);
	InTelemetry->bUsesTmvImageSequence |= (TranscodeOutputFormat == ETmvMediaTranscodeOutputFormat::FileSequence);
}

void UMovieGraphTmvEncoderNode::InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes)
{
	if (bEnableBurnIn)
	{
		UMovieGraphOutputBurnInNode* BurnInNode = NewObject<UMovieGraphOutputBurnInNode>(InEvaluatedConfig);
		BurnInNode->bOverride_OutputName = true;
		BurnInNode->bOverride_FileNameFormat = true;
		BurnInNode->bOverride_OutputRestriction = true;
		BurnInNode->bOverride_LayerNameFormat = true;
		BurnInNode->bOverride_BurnInClass = true;
		BurnInNode->bOverride_bCompositeOntoFinalImage = true;
		BurnInNode->OutputName = TEXT("TMV");
		BurnInNode->FileNameFormat = BurnInFileNameFormat;
		BurnInNode->OutputRestriction = FSoftClassPath(GetClass());
		BurnInNode->LayerNameFormat = TEXT("");	// Unused
		BurnInNode->BurnInClass = BurnInClass;
		BurnInNode->bCompositeOntoFinalImage = bCompositeOntoFinalImage;

		OutInjectedNodes.Add(BurnInNode);
	}
}

bool UMovieGraphTmvEncoderNode::IsOcioEnabledRuntime() const
{
	return OCIOConfiguration.bIsEnabled;
}

#if WITH_EDITOR
bool UMovieGraphTmvEncoderNode::IsOcioEnabledEditor() const
{
	// If OCIOConfiguration is connected via a pin, we can't resolve its value at editor time.
	// Allow editing and let the runtime evaluation handle the actual OCIO state.
	if (const UMovieGraphPin* Pin = GetInputPin(GET_MEMBER_NAME_CHECKED(UMovieGraphTmvEncoderNode, OCIOConfiguration)))
	{
		if (Pin->IsConnected())
		{
			return false;
		}
	}

	if (bOverride_OCIOConfiguration)
	{
		return OCIOConfiguration.bIsEnabled;
	}
	// Default when not overriden should be false.
	return false;
}
#endif