// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "ApvMediaTypes.h"
#include "Encoder/TmvMediaEncoderOptions.h"
#include "Graph/MovieGraphNode.h"
#include "Graph/Nodes/MovieGraphVideoOutputNode.h"
#include "Transcoder/TmvMediaTranscodeJob.h"

#include "MovieGraphTmvEncoderNode.generated.h"

class UTmvMediaTranscodeJob;

/**
 * Selection of the output types, defined as the extension (apv1, tmv). 
 * @remark This enum is different conceptually then the internal tmv output format which is 
 * a muxer enum (container, file sequence) not directly related to file extensions.
 */
UENUM(BlueprintType)
enum class ETmvEncoderNodeOutputType : uint8
{
	/** Output each frame as a separate file in Apv1 format. */
	Apv1ImageSequence UMETA(DisplayName="APV1 Image Sequence"),
	/** Mux freams into a single tmv container file. */
	TmvVideoFile UMETA(DisplayName="TMV Video File"),
};

/** A node which can output TMV (Tiled-Mipmap Video) files using the APV encoder. */
UCLASS(BlueprintType)
class UMovieGraphTmvEncoderNode final : public UMovieGraphVideoOutputNode, public IMovieGraphEvaluationNodeInjector
{
	GENERATED_BODY()

public:
	UMovieGraphTmvEncoderNode();

#if WITH_EDITOR
	virtual FText GetNodeTitle(const bool bGetDescriptive = false) const override;
	virtual FText GetMenuCategory() const override;
	virtual FText GetKeywords() const override;
	virtual FLinearColor GetNodeTitleColor() const override;
	virtual FSlateIcon GetIconAndTint(FLinearColor& OutColor) const override;

	//~ Begin UObject interface
	virtual bool CanEditChange(const FProperty* InProperty) const override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	//~ End UObject interface
#endif

protected:
	//~ Begin UMovieGraphVideoOutputNode
	virtual TUniquePtr<MovieRenderGraph::IVideoCodecWriter> Initialize_GameThread(const FMovieGraphVideoNodeInitializationContext& InInitializationContext) override;
	virtual bool Initialize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	virtual void WriteFrame_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter, FImagePixelData* InPixelData, TArray<FMovieGraphPassData>&& InCompositePasses, TObjectPtr<UMovieGraphEvaluatedConfig> InEvaluatedConfig, const FString& InBranchName) override;
	virtual void BeginFinalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	virtual void Finalize_EncodeThread(MovieRenderGraph::IVideoCodecWriter* InWriter) override;
	virtual const TCHAR* GetFilenameExtension() const override;
	virtual bool IsAudioSupported() const override;
	//~ End UMovieGraphVideoOutputNode

public:
	//~ Begin UMovieGraphSettingNode
	virtual void UpdateTelemetry(FMoviePipelineShotRenderTelemetry* InTelemetry) const override;
	//~ End UMovieGraphSettingNode

	//~ Begin IMovieGraphEvaluationNodeInjector
	virtual void InjectNodesPostEvaluation(const FName& InBranchName, UMovieGraphEvaluatedConfig* InEvaluatedConfig, TArray<UMovieGraphSettingNode*>& OutInjectedNodes) override;
	//~ End IMovieGraphEvaluationNodeInjector

protected:
	/** 
	 * Returns true if the OCIO configuration is enabled in the runtime (evaluated) context.
	 */
	bool IsOcioEnabledRuntime() const;
	
#if WITH_EDITOR
	/** 
	 * Returns true if the OCIO configuration is enabled in the editor context (i.e. called on the edited node, not evaluated).
	 */
	bool IsOcioEnabledEditor() const;
#endif
	
	struct FTmvCodecWriter : public MovieRenderGraph::IVideoCodecWriter
	{
		/** Transcode job that will convert and encode the produced frames. */
		TStrongObjectPtr<UTmvMediaTranscodeJob> TranscodeJob;
	};

	/** The pipeline that is running this node. */
	TWeakObjectPtr<UMovieGraphPipeline> CachedPipeline;

public:
	// --- Override bits ---

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OutputType : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableMipMapping : 1;
	
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_TileSize : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Profile : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Band : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_Preset : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_NumThreads : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DestinationColorSpace : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_DestinationEncoding : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_ReferenceWhite : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_YuvMatrix : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_YuvMatrixRange : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OCIOConfiguration : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_OCIOContext : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bEnableBurnIn : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInClass : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_bCompositeOntoFinalImage : 1;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Overrides, meta = (InlineEditConditionToggle))
	uint8 bOverride_BurnInFileNameFormat : 1;

	// --- Output Format ---

	/** Select whether to output a single TMV container file or individual files per frame. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "File Output", meta = (EditCondition = "bOverride_OutputType", DisplayPriority = 100))
	ETmvEncoderNodeOutputType OutputType = ETmvEncoderNodeOutputType::TmvVideoFile;

	// --- APV Encoder Settings ---

	/**
	 * If checked, mip maps will be generated and encoded in the final destination.
	 * The encoder may restrict the generated mip chain depending on the format requirements.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "APV", meta = (EditCondition = "bOverride_bEnableMipMapping"))
	bool bEnableMipMapping = true;
	
	/** 
	 * Dimensions of a tile in pixels.
	 * The tile size determines parallelism and performance. Frames are divided into rectangular tiles that 
	 * can be encoded and decoded simultaneously, enabling high throughput on multicore processors.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "APV", meta = (EditCondition = "bOverride_TileSize", ClampMin = "0.0"))
	FIntPoint TileSize = FIntPoint(256, 256);

	/**
	 * Specify the color/chroma profile, includes bitdepth.
	 * Input video will be converted to the given profile.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "APV", meta = (EditCondition = "bOverride_Profile"))
	EApvMediaProfile Profile = EApvMediaProfile::YCbCr422_10;

	/**
	 * Coded data rate band setting.
	 * If the bitrate is not specified otherwise, this determines the max coded data rate (higher band leads to higher bit rate).
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "APV", meta = (EditCondition = "bOverride_Band"))
	EApvMediaBand Band = EApvMediaBand::Band2;

	/** Specify the encoding optimization level. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "APV", meta = (EditCondition = "bOverride_Preset"))
	EApvMediaPreset Preset = EApvMediaPreset::Medium;

	/** Number of worker threads to encode tiles (0 = automatically use the available number of cores). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "APV", meta = (EditCondition = "bOverride_NumThreads", ClampMin = "0"))
	int32 NumThreads = 0;

	// --- Color Management ---

	/** Specify the destination color space. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Management", meta = (EditCondition = "bOverride_DestinationColorSpace", ValidEnumValues = "TCS_sRGB, TCS_Rec2020"))
	ETextureColorSpace DestinationColorSpace = ETextureColorSpace::TCS_sRGB;

	/** Specify the destination color encoding. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Management", meta = (EditCondition = "bOverride_DestinationEncoding", ValidEnumValues = "Linear, sRGB, ST2084, SLog3, HLG"))
	ETmvMediaEncoderEncoding DestinationEncoding = ETmvMediaEncoderEncoding::sRGB;

	/** Reference-white standard used to scale UE scene-linear 1.0 to diffuse white. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Management", meta = (EditCondition = "bOverride_ReferenceWhite && (DestinationEncoding == ETmvMediaEncoderEncoding::ST2084 || DestinationEncoding == ETmvMediaEncoderEncoding::HLG)", EditConditionHides))
	ETmvMediaEncoderReferenceWhite ReferenceWhite = ETmvMediaEncoderReferenceWhite::BT2408;

	/** Specify the YUV encoding color matrix. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Management", meta = (EditCondition = "bOverride_YuvMatrix"))
	ETmvMediaEncoderColorMatrix YuvMatrix = ETmvMediaEncoderColorMatrix::Rec709;

	/** Specify the YUV encoding color matrix range. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Color Management", meta = (EditCondition = "bOverride_YuvMatrixRange"))
	ETmvMediaEncoderColorMatrixRange YuvMatrixRange = ETmvMediaEncoderColorMatrixRange::Full;

	// --- OCIO ---

	/**
	* OCIO configuration/transform settings.
	*
	* If OCIO transform is enabled, the builtin color transform in Color Management will be ignored.
	*
	* Note: There are differences from the previous implementation in MRQ given that we are now doing CPU-side processing.
	* 1) This feature only works on desktop platforms when the OpenColorIO library is available.
	* 2) Users are now responsible for setting the renderer output space to Final Color (HDR) in Linear Working Color Space (SCS_FinalColorHDR) by
	*    disabling the Tone Curve setting on the renderer node.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", DisplayName="OCIO Configuration", meta = (DisplayAfter = "FileNameFormat", EditCondition = "bOverride_OCIOConfiguration"))
	FOpenColorIODisplayConfiguration OCIOConfiguration;

	/**
	* OCIO context of key-value string pairs, typically used to apply shot-specific looks (such as a CDL color correction, or a 1D grade LUT).
	*
	* Notes:
	* 1) If a configuration asset base context was set, it remains active but can be overridden here with new key-values.
	* 2) Format tokens such as {shot_name} are supported and will get resolved before submission.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "OCIO", DisplayName = "OCIO Context", meta = (DisplayAfter = "OCIOConfiguration", EditCondition = "bOverride_OCIOContext"))
	TMap<FString, FString> OCIOContext;

	// --- Burn In ---

	/** If true, this output node will also generate a burn-in. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bEnableBurnIn"))
	bool bEnableBurnIn = false;

	/** The widget that the burn-in should use. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Burn In", meta=(MetaClass="/Script/MovieRenderPipelineCore.MovieGraphBurnInWidget", EditCondition="bOverride_BurnInClass"))
	FSoftClassPath BurnInClass = FSoftClassPath(TEXT("/MovieRenderPipeline/Blueprints/Graph/DefaultGraphBurnIn.DefaultGraphBurnIn_C"));

	/** If true, the burn-in that's generated will be composited onto this output. Otherwise, the burn-in will be written to a different file. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_bCompositeOntoFinalImage"))
	bool bCompositeOntoFinalImage = true;

	/** The file name format used for writing out the burn-in video (if not composited). */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Burn In", meta = (EditCondition = "bOverride_BurnInFileNameFormat", MrgTokenAutocomplete))
	FString BurnInFileNameFormat = TEXT("{sequence_name}.{layer_name}.{renderer_name}.{frame_number}");
};