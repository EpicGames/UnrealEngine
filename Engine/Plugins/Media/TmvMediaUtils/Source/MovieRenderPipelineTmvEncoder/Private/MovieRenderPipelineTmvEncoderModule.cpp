// Copyright Epic Games, Inc. All Rights Reserved.

#include "Modules/ModuleManager.h"

class FMovieRenderPipelineTmvEncoderModule : public IModuleInterface
{
public:
	/*~ Begin IModuleInterface interface */
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	/*~ End IModuleInterface interface */
};

void FMovieRenderPipelineTmvEncoderModule::StartupModule()
{
}

void FMovieRenderPipelineTmvEncoderModule::ShutdownModule()
{
}

IMPLEMENT_MODULE(FMovieRenderPipelineTmvEncoderModule, MovieRenderPipelineTmvEncoder)