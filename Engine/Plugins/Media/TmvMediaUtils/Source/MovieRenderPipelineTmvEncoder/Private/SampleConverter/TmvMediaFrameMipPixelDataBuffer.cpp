// Copyright Epic Games, Inc. All Rights Reserved.

#include "TmvMediaFrameMipPixelDataBuffer.h"

#include "ImageCore.h"
#include "ImagePixelData.h"
#include "RHIResources.h"
#include "Utils/TmvMediaFrameUtils.h"

namespace UE::TmvMediaUtils
{
	void* GetPixelDataPointer(const FImagePixelData& InImage)
	{
		const void* Data = nullptr;
		int64 DataSize;
		if (InImage.GetRawData(Data, DataSize))
		{
			return const_cast<void*>(Data);
		}
		return nullptr;
	}
}

FTmvMediaFrameMipPixelDataBuffer::FTmvMediaFrameMipPixelDataBuffer() = default;

FTmvMediaFrameMipPixelDataBuffer::FTmvMediaFrameMipPixelDataBuffer(TUniquePtr<FImagePixelData>&& InImage, int32 InMipLevel)
{
	if (InImage)
	{
		FImageView ImageView = InImage->GetImageView();
		AllocatedBufferSize = ImageView.GetImageSizeBytes();
		UE::TmvMedia::FrameUtils::PopulateMipInfoFromImageInfo(InMipLevel, ImageView, MipInfo);
		PlaneImages.Add(MoveTemp(InImage));
	}
}

FTmvMediaFrameMipPixelDataBuffer::~FTmvMediaFrameMipPixelDataBuffer() = default;

bool FTmvMediaFrameMipPixelDataBuffer::RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo)
{
	// todo: currently not used. Implement if needed.
	ensureMsgf(false, TEXT("FTmvMediaFrameMipPixelDataBuffer::RequestAllocation is not implemented."));
	return false;
}

void* FTmvMediaFrameMipPixelDataBuffer::GetMappedBuffer()
{
	// Only return the first plane buffer if we have one plane to be compliant with
	// the api that should return one contiguous buffer.
	if (PlaneImages.Num() == 1 && PlaneImages.IsValidIndex(0) && PlaneImages[0])
	{
		return UE::TmvMediaUtils::GetPixelDataPointer(*PlaneImages[0]);
	}
	return nullptr;
}

void* FTmvMediaFrameMipPixelDataBuffer::GetPlaneBufferForComponent(int32 InComponentIndex)
{
	// Use the mip info to find the plane index for this component
	const int32 PlaneIndex = MipInfo.GetPlaneIndexForComponent(InComponentIndex);
	if (PlaneImages.IsValidIndex(PlaneIndex) && PlaneImages[PlaneIndex])
	{
		return UE::TmvMediaUtils::GetPixelDataPointer(*PlaneImages[PlaneIndex]);
	}
	return nullptr;
}

FShaderResourceViewRHIRef FTmvMediaFrameMipPixelDataBuffer::GetShaderResourceView(int32 InComponentIndex) const
{
	return nullptr;
}