// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "SampleConverter/TmvMediaFrameMipBuffer.h"

struct FImagePixelData;

/**
 * Implementation of texture cpu buffering backed by FImagePixelData.
 */
class FTmvMediaFrameMipPixelDataBuffer : public FTmvMediaFrameMipBuffer
{
public:
	/* Constructor */
	FTmvMediaFrameMipPixelDataBuffer();

	/** Construct from an existing image, using the image as backing buffer. */
	FTmvMediaFrameMipPixelDataBuffer(TUniquePtr<FImagePixelData>&& InImage, int32 InMipLevel);

	/** Destructor to clean up the internal buffer data. */
	virtual ~FTmvMediaFrameMipPixelDataBuffer() override;

	//~ Begin FTmvMediaFrameMipBuffer
	virtual bool RequestAllocation(const FTmvMediaFrameMipInfo& InMipInfo) override;
	virtual void WaitAllocation() override {}
	virtual bool TryUpdateMipInfo(const FTmvMediaFrameMipInfo& InMipInfo) override { return false; }
	virtual void* GetMappedBuffer() override;
	virtual void* GetPlaneBufferForComponent(int32 InComponentIndex) override;
	virtual FShaderResourceViewRHIRef GetShaderResourceView(int32 InComponentIndex) const override;
	virtual void CopyTileRegions(int32 InFrameId, const FTmvMediaFrameMipInfo& InTmvMipInfo, TConstArrayView<FIntRect> InTileRegions, TConstArrayView<FTmvMediaShaderTileDesc> InTileInfos) override {}
	virtual bool IsValidForRendering() const override
	{
		return false;	// This buffer type can't be used for rendering.
	}
	//~ End FTmvMediaFrameMipBuffer

private:
	/** Image buffer for each plane. */
	TArray<TUniquePtr<FImagePixelData>> PlaneImages;
};