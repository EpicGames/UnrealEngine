{
	"FileVersion" : 3,
	"Version" : 1,
	"VersionName" : "1.0",
	"FriendlyName" : "Tiled Mipmap Video Utilities",
	"Description" : "Additional tools for the tiled-mipmap video (TMV) framework.\nImplements Movie Render Graph (MRG) integration.",
	"Category" : "Media Players",
	"CreatedBy" : "Epic Games, Inc.",
	"CreatedByURL" : "https://epicgames.com",
	"DocsURL" : "",
	"MarketplaceURL" : "",
	"SupportURL" : "",
	"EnabledByDefault" : false,
	"CanContainContent" : false,
	"IsBetaVersion" : true,
	"Installed" : false,
	"Plugins": [
		{
			"Name": "TmvMedia",
			"Enabled": true,
			"Optional": false
		},
		{
			"Name": "MovieRenderPipeline",
			"Enabled": true,
			"Optional": false
		}
	],
	"Modules" :
	[
		{
			"Name" : "MovieRenderPipelineTmvEncoder",
			"Type" : "Runtime",
			"LoadingPhase" : "Default",
			"PlatformAllowList" : [ "Win64" ],
			"PlatformArchitectureDenyList": [ "Win64:arm64" ]
		}
	]
}
